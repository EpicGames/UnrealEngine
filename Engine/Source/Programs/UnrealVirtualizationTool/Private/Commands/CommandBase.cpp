// Copyright Epic Games, Inc. All Rights Reserved.

#include "CommandBase.h"

#include "HAL/FileManager.h"
#include "ISourceControlModule.h"
#include "ISourceControlProvider.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "Misc/ScopeExit.h"
#include "SourceControlInitSettings.h"
#include "SourceControlOperations.h"
#include "UnrealVirtualizationTool.h"

namespace UE::Virtualization
{

/** Utility for testing if a file path resolves to a valid package file or not */
bool IsPackageFile(const FString FilePath)
{
	// ::IsPackageExtension requires a TCHAR so we cannot use FPathViews here
	const FString Extension = FPaths::GetExtension(FilePath);

	// Currently we don't virtualize text based assets so no call to FPackageName::IsTextPackageExtension
	return FPackageName::IsPackageExtension(*Extension);
}

FCommand::FCommand(FStringView InCommandName)
	: CommandName(InCommandName)
{

}

FCommand::~FCommand()
{

}

void FCommand::ParseCommandLine(const TCHAR* CmdLine, TArray<FString>& Tokens, TArray<FString>& Switches)
{
	// TODO: Taken from UCommandlet, maybe consider moving this code to a general purpose utility. We 
	// could also make a version that returns TArray<FStringView>

	FString NextToken;
	while (FParse::Token(CmdLine, NextToken, false))
	{
		if (**NextToken == TCHAR('-'))
		{
			new(Switches) FString(NextToken.Mid(1));
		}
		else
		{
			new(Tokens) FString(NextToken);
		}
	}
}

FCommand::EPathResult FCommand::ParseSwitchForPaths(const FString& Switch, TArray<FString>& OutPackages)
{
	FString Path;
	if (FParse::Value(*Switch, TEXT("Package="), Path))
	{
		FPaths::NormalizeFilename(Path);

		if (!FPackageName::IsPackageFilename(Path))
		{
			UE_LOGF(LogVirtualizationTool, Error, "Requested package file '%ls' is not a valid package filename", *Path);
			return EPathResult::Error;
		}

		if (!IFileManager::Get().FileExists(*Path))
		{
			UE_LOGF(LogVirtualizationTool, Error, "Could not find the requested package file '%ls'", *Path);
			return EPathResult::Error;
		}

		OutPackages.Add(Path);
		return EPathResult::Success;
	}
	else if (FParse::Value(*Switch, TEXT("PackageDir="), Path) || FParse::Value(*Switch, TEXT("PackageFolder="), Path))
	{
		// Note that 'PackageFolder' is the switch used by the resave commandlet, so allowing it here for compatibility purposes
		FPaths::NormalizeFilename(Path);
		if (IFileManager::Get().DirectoryExists(*Path))
		{
			IFileManager::Get().IterateDirectoryRecursively(*Path, [&OutPackages](const TCHAR* Path, bool bIsDirectory)
				{
					if (!bIsDirectory && FPackageName::IsPackageFilename(Path))
					{
						FString FilePath(Path);
						FPaths::NormalizeFilename(FilePath);

						OutPackages.Add(MoveTemp(FilePath));
					}

					return true; // Continue
				});

			return EPathResult::Success;
		}
		else
		{
			UE_LOGF(LogVirtualizationTool, Error, "Could not find the requested directory '%ls'", *Path);
			return EPathResult::Error;
		}
	}

	return EPathResult::NotFound;
}

bool FCommand::TryConnectToSourceControl(FStringView ClientSpecName)
{
	if (SCCProvider != nullptr)
	{
		// Already connected so just return
		return true;
	}

	if (FPaths::IsProjectFilePathSet())
	{
		UE_LOGF(LogVirtualizationTool, Log, "\t\tTrying to connect to source control with project settings...");

		// If the project has been set then we can use the default source control provider
		if (!ISourceControlModule::Get().GetProvider().IsEnabled())
		{
			ISourceControlModule::Get().SetProvider(FName("Perforce"));
		}

		SCCProvider = &ISourceControlModule::Get().GetProvider();
		SCCProvider->Init(true);
	}
	else
	{
		UE_LOGF(LogVirtualizationTool, Log, "\t\tTrying to connect to source control with default settings...");

		// If we do not have a project set we need to create our own provider. We make the assumption that
		// we should use perforce since the only reason we should need a provider without a project is if
		// we are parsing a perforce changelist for files to operate on.
		FSourceControlInitSettings SCCSettings(FSourceControlInitSettings::EBehavior::OverrideAll);
		SCCSettings.SetConfigBehavior(FSourceControlInitSettings::EConfigBehavior::ReadOnly);
		SCCSettings.SetCmdLineFlags(FSourceControlInitSettings::ECmdLineFlags::ReadAll);
		SCCSettings.AddSetting(TEXT("P4Client"), ClientSpecName);

		OwnedSCCProvider = ISourceControlModule::Get().CreateProvider(FName("Perforce"), TEXT("UnrealVirtualizationTool"), SCCSettings);
		if (OwnedSCCProvider.IsValid())
		{
			SCCProvider = OwnedSCCProvider.Get();
			SCCProvider->Init(true);
		}
		else
		{
			UE_LOGF(LogVirtualizationTool, Error, "\t\tFailed to instantiate a perforce revision control connection");
			return false;
		}
	}

	if (SCCProvider->IsAvailable())
	{
		TMap<ISourceControlProvider::EStatus, FString> StatusMap = SCCProvider->GetStatus();

		if (FString* Server = StatusMap.Find(ISourceControlProvider::EStatus::Port))
		{
			UE_LOGF(LogVirtualizationTool, Log, "\t\tSuccessfully connected to server '%ls'", **Server);
		}
		
		return true;
	}
	else
	{
		UE_LOGF(LogVirtualizationTool, Error, "\tFailed to establish a perforce connection");
		return false;
	}
}

bool FCommand::TryParseChangelist(FStringView ClientSpecName, FStringView ChangelistNumber, TArray<FString>& OutPackages, FSourceControlChangelistPtr* OutChangelist)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(TryParseChangelist);

	UE_LOGF(LogVirtualizationTool, Display, "\tAttempting to parse changelist '%.*ls' in workspace '%.*ls'", 
		ChangelistNumber.Len(), ChangelistNumber.GetData(), 
		ClientSpecName.Len(), ClientSpecName.GetData());

	if (!TryConnectToSourceControl(ClientSpecName))
	{
		return false;
	}

	if (SCCProvider == nullptr)
	{
		UE_LOGF(LogVirtualizationTool, Error, "No valid source control connection found!");
		return false;
	}

	if (!SCCProvider->UsesChangelists())
	{
		UE_LOGF(LogVirtualizationTool, Error, "The source control provider does not support the use of changelists");
		return false;
	}

	TArray<FSourceControlChangelistRef> Changelists = SCCProvider->GetChangelists(EStateCacheUsage::ForceUpdate);
	if (Changelists.IsEmpty())
	{
		UE_LOGF(LogVirtualizationTool, Error, "Failed to find any changelists");
		return false;
	}

	// TODO: At the moment we have to poll for all changelists under the current workspace and then iterate over
	// them to find the one we want.
	// We need to add better support in the API to go from a changelist number to a FSourceControlChangelistRef
	// directly.
	TArray<FSourceControlChangelistStateRef> ChangelistsStates;
	if (SCCProvider->GetState(Changelists, ChangelistsStates, EStateCacheUsage::Use) != ECommandResult::Succeeded)
	{
		UE_LOGF(LogVirtualizationTool, Error, "Failed to find changelist data");
		return false;
	}

	for (FSourceControlChangelistStateRef& ChangelistState : ChangelistsStates)
	{
		const FText DisplayText = ChangelistState->GetDisplayText();

		if (ChangelistNumber == DisplayText.ToString())
		{
			TSharedRef<FUpdatePendingChangelistsStatus, ESPMode::ThreadSafe> Operation = ISourceControlOperation::Create<FUpdatePendingChangelistsStatus>();

			FSourceControlChangelistRef Changelist = ChangelistState->GetChangelist();
			Operation->SetChangelistsToUpdate(MakeArrayView(&Changelist, 1));
			Operation->SetUpdateFilesStates(true);

			if (SCCProvider->Execute(Operation, EConcurrency::Synchronous) != ECommandResult::Succeeded)
			{
				UE_LOGF(LogVirtualizationTool, Error, "Failed to find the files in changelist '%.*ls'", ChangelistNumber.Len(), ChangelistNumber.GetData());
				return false;
			}

			const TArray<FSourceControlStateRef>& FilesinChangelist = ChangelistState->GetFilesStates();

			UE_LOGF(LogVirtualizationTool, Log, "\tFound %d files in the changelist", FilesinChangelist.Num());

			for (const FSourceControlStateRef& FileState : FilesinChangelist)
			{
				if (!IsPackageFile(FileState->GetFilename()))
				{
					UE_LOGF(LogVirtualizationTool, Log, "\tIgnoring non-package file '%ls'", *FileState->GetFilename());
					continue;
				}

				if (FileState->IsDeleted())
				{
					UE_LOGF(LogVirtualizationTool, Verbose, "\tIgnoring package marked for delete '%ls'", *FileState->GetFilename());
					continue;
				}

				if (FileState->IsIgnored())
				{
					UE_LOGF(LogVirtualizationTool, Verbose, "\tIgnoring package marked for ignore '%ls'", *FileState->GetFilename());
					continue;
				}

				OutPackages.Add(FileState->GetFilename());
			}

			if (OutChangelist != nullptr)
			{
				*OutChangelist = Changelist;
			}

			return true;
		}
	}

	UE_LOGF(LogVirtualizationTool, Error, "Failed to find the changelist '%.*ls'", ChangelistNumber.Len(), ChangelistNumber.GetData());

	return false;
}

FString FCommand::FindClientSpecForChangelist(FStringView ChangelistNumber)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FindClientSpecForChangelist);

	UE_LOGF(LogVirtualizationTool, Display, "\tAttempting to find the workspace for '%.*ls'",
		ChangelistNumber.Len(), ChangelistNumber.GetData());

	if (!TryConnectToSourceControl())
	{
		return FString();
	}

	if (SCCProvider == nullptr)
	{
		UE_LOGF(LogVirtualizationTool, Error, "No valid source control connection found!");
		return FString();
	}

	if (!SCCProvider->UsesChangelists())
	{
		UE_LOGF(LogVirtualizationTool, Error, "The source control provider does not support the use of changelists");
		return FString();
	}

	TSharedRef<FGetChangelistDetails> Operation = ISourceControlOperation::Create<FGetChangelistDetails>(ChangelistNumber);

	if (SCCProvider->Execute(Operation, EConcurrency::Synchronous) == ECommandResult::Succeeded && !Operation->GetChangelistDetails().IsEmpty())
	{
		const FString* ClientSpec = Operation->GetChangelistDetails()[0].Find(TEXT("Client"));
		if (ClientSpec != nullptr)
		{
			return *ClientSpec;
		}
		else
		{ 
			UE_LOGF(LogVirtualizationTool, Error, "Unable to find the 'client' field for the changelist '%.*ls'", ChangelistNumber.Len(), ChangelistNumber.GetData());
			return FString();
		}
	}
	else
	{
		UE_LOGF(LogVirtualizationTool, Error, "Failed to find details for the changelist '%.*ls'", ChangelistNumber.Len(), ChangelistNumber.GetData());
		return FString();
	}
}

} //namespace UE::Virtualization 
