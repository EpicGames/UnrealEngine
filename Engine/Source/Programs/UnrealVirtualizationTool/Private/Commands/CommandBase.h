// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "Dom/JsonObject.h"
#include "Serialization/JsonSerializerMacros.h"
#include "Serialization/JsonWriter.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"

struct FJsonSerializerBase;

class ISourceControlProvider;
class ISourceControlChangelist;

typedef TSharedPtr<ISourceControlChangelist, ESPMode::ThreadSafe> FSourceControlChangelistPtr;


namespace UE::Virtualization
{

class FProject;

// The JSON serialization macros do not support inheritance properly as far as I can tell, this helps work around that.
// Currently it stores all levels of the inheritance chain in the same json object, I'd want to change this to have an
// object per level before considering moving this to JsonSerializerMacros.h
#define JSON_SERIALIZE_PARENT(ParentType) \
	ParentType::Serialize(Serializer, true);

struct FCommandOutput : public FJsonSerializable
{
	FCommandOutput() = default;
	FCommandOutput(FStringView InProjectName)
		: ProjectName(InProjectName)
	{
	}

	BEGIN_JSON_SERIALIZER
		JSON_SERIALIZE("ProjectName", ProjectName);
	END_JSON_SERIALIZER

	FString ProjectName;
};

/** The base class to derive new commands from */
class FCommand
{
public:
	FCommand(FStringView InCommandName);
	virtual ~FCommand();

	virtual bool Initialize(const TCHAR* CmdLine) = 0;

	void ToJson(TSharedRef<TJsonWriter<> >& JsonWriter) const
	{
		FJsonSerializerWriter<> Serializer(JsonWriter);
		const_cast<FCommand*>(this)->Serialize(Serializer);
	}

	bool FromJson(TSharedPtr<FJsonObject> JsonObject)
	{
		if (JsonObject.IsValid())
		{
			FJsonSerializerReader Serializer(JsonObject);
			Serialize(Serializer);
			return true;
		}
		return false;
	}

	virtual void Serialize(FJsonSerializerBase& Serializer) = 0;

	/**
	 * Check to see if the provided project can be run with the current command.
	 * 
	 * @Project	The project to be validated
	 * @return	True if the project is allowed to be processed by the command and false if it is not.
	 */
	virtual bool IsProjectValidForCommand(const FProject& Project) const = 0;

	/** 
	* Called when a project needs to be processed by the command. The output parameter is used to
	* return output that will eventually be passed to ::ProcessOutput. If the command does not 
	* produce any output needed by ::ProcessOutput then the parameter may be ignored.
	*/
	virtual bool ProcessProject(const FProject& Project, TUniquePtr<FCommandOutput>& Output) = 0;

	/** 
	 * Called after all projects have been processed. This will always be called by the original
	 * processes and never called by a spawned child process. If the command produces output when
	 * processing a project then the CmdOutputArray will be populated, otherwise it will be empty.
	 * It is assumed that each command knows how to cast the FCommandOutput to the correct type.
	 */
	virtual bool ProcessOutput(const TArray<TUniquePtr<FCommandOutput>>& CmdOutputArray) = 0;

	/** 
	 * Derived command classes should return a new instance of the command output that it can produce.
	 * This will be called when we are parsing child process output files while attempting to reconstruct
	 * the output of a child process.
	 * Commands that do not return any output can return nullptr. 
	 */
	virtual TUniquePtr<FCommandOutput> CreateOutputObject() const = 0;

	virtual const TArray<FString>& GetPackages() const = 0;

	const FString& GetName() const
	{
		return CommandName;
	}

protected: // Common commandline parsing code
	
	static void ParseCommandLine(const TCHAR* CmdLine, TArray<FString>& Tokens, TArray<FString>& Switches);

	enum EPathResult : int8
	{
		/** The switch was a valid package/path but parsing it resulted in an error */
		Error = -1,
		/** The switch was not a valid package/path */
		NotFound = 0,
		/** The switch was a valid package/path and was successfully parsed */
		Success = 1
	};

	static EPathResult ParseSwitchForPaths(const FString& Switch, TArray<FString>& OutPackages);

protected: // Common SourceControl Code
	/**
	 * Creates a new ISourceControlProvider that can be used to perforce source control operations. This provider
	 * will remain valid until reset or the command is completed.
	 * 
	 * @param ClientSpecName	The client spec to use when connecting. If left blank we will use a default
	 *							spec, which will likely be auto determined by the system.
	 * @return True if the provider was created, otherwise false.
	 */
	bool TryConnectToSourceControl(FStringView ClientSpecName);
	bool TryConnectToSourceControl()
	{
		return TryConnectToSourceControl(FStringView());
	}

	bool TryParseChangelist(FStringView ClientSpecName, FStringView ChangelistNumber, TArray<FString>& OutPackages, FSourceControlChangelistPtr* OutChangelist);

	FString FindClientSpecForChangelist(FStringView ChangelistNumber);

private:
	FString CommandName;

	/*
	 * Sometimes the command will have to create and own the source control provider and be responsible for
	 * cleaning it up once the command has finished. In these cases we store the pointer to the provider
	 * that we own, here.
	 */
	TUniquePtr<ISourceControlProvider> OwnedSCCProvider;

protected:

	/*
	 * Pointer to the source control provider that the command should use.This can either be to a provider
	 * that the command owns in 'OwnedSCCProvider' or one provided for us by the source control api.
	 */
	ISourceControlProvider* SCCProvider = nullptr;

	
};

} // namespace UE::Virtualization