// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commands/RehydrateCommand.h"

#include "ISourceControlProvider.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/PackageName.h"
#include "Misc/Parse.h"
#include "Misc/ScopeExit.h"
#include "ProjectFiles.h"
#include "UnrealVirtualizationTool.h"
#include "Virtualization/VirtualizationSystem.h"

namespace UE::Virtualization
{

FRehydrateCommand::FRehydrateCommand(FStringView CommandName)
	: FCommand(CommandName)
{

}

void FRehydrateCommand::PrintCmdLineHelp()
{
	UE_LOGF(LogVirtualizationTool, Display, "<ProjectFilePath> -Mode=Rehydrate -Package=<string>");
	UE_LOGF(LogVirtualizationTool, Display, "<ProjectFilePath> -Mode=Rehydrate -PackageDir=<string>");
	UE_LOGF(LogVirtualizationTool, Display, "<ProjectFilePath> -Mode=Rehydrate -Changelist=<number>");
	UE_LOGF(LogVirtualizationTool, Display, "");
}

bool FRehydrateCommand::Initialize(const TCHAR* CmdLine)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FRehydrateCommand::Initialize);

	// Note that we haven't loaded any projects config files and so don't really have
	// any valid project mount points so we cannot use FPackagePath or FPackageName
	// and expect to find anything!

	TArray<FString> Tokens;
	TArray<FString> Switches;

	ParseCommandLine(CmdLine, Tokens, Switches);
	
	FString SourceChangelistNumber;
	FString SwitchValue;

	for (const FString& Switch :  Switches)
	{
		EPathResult Result = ParseSwitchForPaths(Switch, Packages);
		if (Result == EPathResult::Error)
		{
			return false;
		}
		else if (Result == EPathResult::Success)
		{
			continue; // If we already matched the switch we don't need to check against any others
		}

		if (FParse::Value(*Switch, TEXT("ClientSpecName="), SwitchValue))
		{
			ClientSpecName = SwitchValue;
			UE_LOGF(LogVirtualizationTool, Display, "\tWorkspace name provided '%ls'", *ClientSpecName);
		}
		else if (FParse::Value(*Switch, TEXT("Changelist="), SwitchValue))
		{
			SourceChangelistNumber = SwitchValue;
		}
		else if (Switch == TEXT("Checkout"))
		{
			bShouldCheckout = true;
		}
	}

	// Process the provided changelist if one was found
	if (!SourceChangelistNumber.IsEmpty())
	{
		// If no client spec was provided we need to find it for the changelist
		// In theory this duplicates a lot of the work found in ::TryParseChangelist
		// but at the moment the FGetChangelistDetails operation is not compatible
		// with the FSourceControlChangelistStateRef/FSourceControlStateRef API
		// so we are stuck with duplication of work.
		if (ClientSpecName.IsEmpty())
		{
			ClientSpecName = FindClientSpecForChangelist(SourceChangelistNumber);
			if (!ClientSpecName.IsEmpty())
			{
				FSourceControlResultInfo Info;
				if (SCCProvider->SwitchWorkspace(ClientSpecName, Info, nullptr) != ECommandResult::Succeeded)
				{
					UE_LOGF(LogVirtualization, Error, "Failed to switch to workspace '%ls'", *ClientSpecName);
					return false;
				}
			}
			else
			{
				UE_LOGF(LogVirtualization, Error, "Count not find a valid workspace for the changelist '%ls'", *SourceChangelistNumber);
				return false;
			}
		}

		if (!TryParseChangelist(ClientSpecName, SourceChangelistNumber, Packages, nullptr))
		{
			UE_LOGF(LogVirtualization, Error, "Failed to find the files in the changelist '%ls'", *SourceChangelistNumber);
			return false;
		}
	}

	return true;
}

void FRehydrateCommand::Serialize(FJsonSerializerBase& Serializer)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FRehydrateCommand::Serialize);

	Serializer.Serialize(TEXT("ShouldCheckout"), bShouldCheckout);
}

bool FRehydrateCommand::IsProjectValidForCommand(const FProject& Project) const
{
	// Currently all project types can be rehydrated but a valid project path is required.
	return !Project.GetProjectFilePath().IsEmpty();
}

bool FRehydrateCommand::ProcessProject(const FProject& Project, TUniquePtr<FCommandOutput>& Output)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FRehydrateCommand::ProcessProject);

	TStringBuilder<128> ProjectName;
	ProjectName << Project.GetProjectName();

	UE_LOGF(LogVirtualizationTool, Display, "\tProcessing package(s) for the project '%ls'...", ProjectName.ToString());

	FConfigFile EngineConfigWithProject;
	if (!Project.TryLoadConfig(EngineConfigWithProject))
	{
		return false;
	}

	Project.RegisterMountPoints();

	ON_SCOPE_EXIT
	{
		Project.UnRegisterMountPoints();
	};

	UE::Virtualization::FInitParams InitParams(ProjectName, EngineConfigWithProject);
	UE::Virtualization::Initialize(InitParams, UE::Virtualization::EInitializationFlags::ForceInitialize);

	ON_SCOPE_EXIT
	{
		UE::Virtualization::Shutdown();
	};

	UE_LOGF(LogVirtualizationTool, Display, "\t\tAttempting to rehydrate packages...");

	const TArray<FString> ProjectPackages = Project.GetAllPackages();

	ERehydrationOptions Options = ERehydrationOptions::None;
	if (bShouldCheckout)
	{
		Options |= ERehydrationOptions::Checkout;

		// Make sure that we have a valid source control connection if we might try to checkout packages
		TryConnectToSourceControl();
	}

	FRehydrationResult Result = UE::Virtualization::IVirtualizationSystem::Get().TryRehydratePackages(ProjectPackages, Options);
	if (!Result.WasSuccessful())
	{
		UE_LOGF(LogVirtualizationTool, Error, "The rehydration process failed with the following errors:");
		for (const FText& Error : Result.Errors)
		{
			UE_LOGF(LogVirtualizationTool, Error, "\t%ls", *Error.ToString());
		}
		return false;
	}

	if (!Result.RehydratedPackages.IsEmpty())
	{
		UE_LOGF(LogVirtualizationTool, Display, "\t\t%d packages were hydrated", Result.RehydratedPackages.Num());
		UE_LOGF(LogVirtualizationTool, Display, "\t\tTotal hydration increase %ls (%ls -> %ls)",
			*FText::AsMemory(Result.PostOperationSize - Result.PreOperationSize).ToString(),
			*FText::AsMemory(Result.PreOperationSize).ToString(),
			*FText::AsMemory(Result.PostOperationSize).ToString());

		if (bShouldCheckout)
		{
			UE_LOGF(LogVirtualizationTool, Display, "\t\t%d packages were checked out of revision control", Result.CheckedOutPackages.Num());
		}
	}
	else
	{
		UE_LOGF(LogVirtualizationTool, Display, "\t\tNo package files were hydrated");
	}

	UE_LOGF(LogVirtualizationTool, Display, "\t\tTime taken %.2f(s)", Result.TimeTaken);
	UE_LOGF(LogVirtualizationTool, Display, "\t\tRehyration of project packages complete!");

	return true;
}

bool FRehydrateCommand::ProcessOutput(const TArray<TUniquePtr<FCommandOutput>>& OutputArray)
{
	// Command has no additional work to do
	return true;
}

TUniquePtr<FCommandOutput> FRehydrateCommand::CreateOutputObject() const
{
	// This command does not create any output
	return TUniquePtr<FCommandOutput>();
}

const TArray<FString>& FRehydrateCommand::GetPackages() const
{
	return Packages;
}

} // namespace UE::Virtualization
