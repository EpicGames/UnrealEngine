// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Commands/CommandBase.h"

namespace UE::Virtualization
{

class FRehydrateCommand final : public FCommand
{
public:
	FRehydrateCommand(FStringView CommandName);
	virtual ~FRehydrateCommand() = default;

	static void PrintCmdLineHelp();
private:
	virtual bool Initialize(const TCHAR* CmdLine) override;

	virtual void Serialize(FJsonSerializerBase& Serializer) override;

	virtual bool IsProjectValidForCommand(const FProject& Project) const final;

	virtual bool ProcessProject(const FProject& Project, TUniquePtr<FCommandOutput>& Output) override;
	virtual bool ProcessOutput(const TArray<TUniquePtr<FCommandOutput>>& OutputArray) override;

	virtual TUniquePtr<FCommandOutput> CreateOutputObject() const override;

	virtual const TArray<FString>& GetPackages() const override;

private:

	FString ClientSpecName;

	TArray<FString> Packages;

	bool bShouldCheckout = false;
};

} // namespace UE::Virtualization