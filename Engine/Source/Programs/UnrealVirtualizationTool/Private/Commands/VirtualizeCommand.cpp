// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commands/VirtualizeCommand.h"

#include "HAL/FileManager.h"
#include "Misc/FileHelper.h"
#include "Misc/ScopeExit.h"
#include "ProjectFiles.h"
#include "UnrealVirtualizationTool.h"
#include "Virtualization/VirtualizationSystem.h"


#include "ISourceControlModule.h"
#include "ISourceControlProvider.h"
#include "SourceControlOperations.h"

namespace UE::Virtualization
{

/** Utility */
TArray<FString> BuildFinalTagDescriptions(const TArray<TUniquePtr<FCommandOutput>>& OutputArray)
{
	TArray<FString> CleanedDescriptions;
	
	for (const TUniquePtr<FCommandOutput>& Output : OutputArray)
	{
		if (Output)
		{
			const FVirtualizeCommandOutput* CmdOutput = (const FVirtualizeCommandOutput*)Output.Get();

			for (const FString& Tag : CmdOutput->DescriptionTags)
			{
				CleanedDescriptions.AddUnique(Tag);
			}
		}
	}

	return CleanedDescriptions;
}

FVirtualizeCommandOutput::FVirtualizeCommandOutput(FStringView InProjectName, const TArray<FText>& InDescriptionTags)
	: FCommandOutput(InProjectName)
{
	DescriptionTags.Reserve(InDescriptionTags.Num());

	for (const FText& Description : InDescriptionTags)
	{
		DescriptionTags.Add(Description.ToString());
	}
}

FVirtualizeCommand::FVirtualizeCommand(FStringView CommandName)
	: FCommand(CommandName)
{

}

void FVirtualizeCommand::PrintCmdLineHelp()
{
	UE_LOGF(LogVirtualizationTool, Display, "<ProjectFilePath> -Mode=Virtualize -Changelist=<number> -Submit [optional]");
	UE_LOGF(LogVirtualizationTool, Display, "<ProjectFilePath> -Mode=Virtualize -Path=<string>");
	UE_LOGF(LogVirtualizationTool, Display, "");
}

bool FVirtualizeCommand::Initialize(const TCHAR* CmdLine)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FVirtualizeCommand::Initialize);

	TArray<FString> Tokens;
	TArray<FString> Switches;

	ParseCommandLine(CmdLine, Tokens, Switches);

	FString SwitchValue;
	for (const FString& Switch : Switches)
	{
		EPathResult Result = ParseSwitchForPaths(Switch, AllPackages);
		if (Result == EPathResult::Error)
		{
			return false;
		}
		else if (Result == EPathResult::Success)
		{
			continue; // If we already matched the switch we don't need to check against any others
		}
		
		if (FParse::Value(*Switch, TEXT("Changelist="), SwitchValue))
		{
			SourceChangelistNumber = SwitchValue;
		}
		else if (Switch == TEXT("Submit"))
		{
			bShouldSubmitChangelist = true;
		}
		else if (Switch == TEXT("Checkout"))
		{
			bShouldCheckout = true;
		}
	}

	// Process the provided changelist if one was found
	if (!SourceChangelistNumber.IsEmpty() )
	{
		// If no client spec was provided we need to find it for the changelist
		// In theory this duplicates a lot of the work found in ::TryParseChangelist
		// but at the moment the FGetChangelistDetails operation is not compatible
		// with the FSourceControlChangelistStateRef/FSourceControlStateRef API
		// so we are stuck with duplication of work.
		if (ClientSpecName.IsEmpty())
		{
			ClientSpecName = FindClientSpecForChangelist(SourceChangelistNumber);
			if (!ClientSpecName.IsEmpty())
			{
				FSourceControlResultInfo Info;
				if (SCCProvider->SwitchWorkspace(ClientSpecName, Info, nullptr) != ECommandResult::Succeeded)
				{
					UE_LOGF(LogVirtualization, Error, "Failed to switch to workspace '%ls'", *ClientSpecName);
					return false;
				}
			}
			else
			{
				UE_LOGF(LogVirtualization, Error, "Count not find a valid workspace for the changelist '%ls'", *SourceChangelistNumber);
				return false;
			}
		}

		if (!TryParseChangelist(ClientSpecName, SourceChangelistNumber, AllPackages, &SourceChangelist))
		{
			UE_LOGF(LogVirtualization, Error, "Failed to find the files in the changelist '%ls'", *SourceChangelistNumber);
			return false;
		}
	}

	return true;
}

void FVirtualizeCommand::Serialize(FJsonSerializerBase& Serializer)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FVirtualizeCommand::Serialize);

	Serializer.Serialize(TEXT("ShouldCheckout"), bShouldCheckout);
}

bool FVirtualizeCommand::IsProjectValidForCommand(const FProject& Project) const
{
	// Only virtualize content in game projects with a valid project path
	return Project.GetProjectType() == EProjectType::GameProject && !Project.GetProjectFilePath().IsEmpty();
}

bool FVirtualizeCommand::ProcessProject(const FProject& Project, TUniquePtr<FCommandOutput>& Output)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FVirtualizeCommand::ProcessProject);

	UE_LOGF(LogVirtualizationTool, Display, "Running the virtualization process...");

	TStringBuilder<128> ProjectName;
	ProjectName << Project.GetProjectName();

	UE_LOGF(LogVirtualizationTool, Display, "\tRunning the virtualization process for the project '%ls'...", ProjectName.ToString());

	FConfigFile EngineConfigWithProject;
	if (!Project.TryLoadConfig(EngineConfigWithProject))
	{
		return false;
	}

	Project.RegisterMountPoints();

	ON_SCOPE_EXIT
	{
		Project.UnRegisterMountPoints();
	};

	UE::Virtualization::FInitParams InitParams(ProjectName, EngineConfigWithProject);
	UE::Virtualization::Initialize(InitParams, UE::Virtualization::EInitializationFlags::ForceInitialize);

	ON_SCOPE_EXIT
	{
		UE::Virtualization::Shutdown();
	};

	if (!UE::Virtualization::IVirtualizationSystem::Get().IsEnabled())
	{
		UE_LOGF(LogVirtualizationTool, Display, "\tVirtualization is not enabled for this project");
		Output = MakeUnique<FVirtualizeCommandOutput>(Project.GetProjectName(), TArray<FText>());
		return true;
	}

	TArray<FString> ProjectPackages = Project.GetAllPackages();

	EVirtualizationOptions Options = EVirtualizationOptions::None;
	if (bShouldCheckout)
	{
		Options |= EVirtualizationOptions::Checkout;

		// Make sure that we have a valid source control connection if we might try to checkout packages
		TryConnectToSourceControl();
	}

	FVirtualizationResult Result = UE::Virtualization::IVirtualizationSystem::Get().TryVirtualizePackages(ProjectPackages, Options);

	if (!Result.WasSuccessful())
	{
		UE_LOGF(LogVirtualizationTool, Error, "The virtualization process failed with the following errors:");
		for (const FText& Error : Result.Errors)
		{
			UE_LOGF(LogVirtualizationTool, Error, "\t%ls", *Error.ToString());
		}
		return false;
	}

	if (!Result.VirtualizedPackages.IsEmpty())
	{
		UE_LOGF(LogVirtualizationTool, Display, "\t\t%d packages were virtualized", Result.VirtualizedPackages.Num());
		UE_LOGF(LogVirtualizationTool, Display, "\t\tTotal Reduction %ls (%ls -> %ls)",
			*FText::AsMemory(Result.PreOperationSize - Result.PostOperationSize).ToString(),
			*FText::AsMemory(Result.PreOperationSize).ToString(),
			*FText::AsMemory(Result.PostOperationSize).ToString());

		if (bShouldCheckout)
		{
			UE_LOGF(LogVirtualizationTool, Display, "\t\t%d packages were checked out of revision control", Result.CheckedOutPackages.Num());
		}
	}
	else
	{
		UE_LOGF(LogVirtualizationTool, Display, "\t\tNo package files were virtualized");
	}

	UE_LOGF(LogVirtualizationTool, Display, "\t\tTime taken %.2f(s)", Result.TimeTaken);
	UE_LOGF(LogVirtualizationTool, Display, "\t\tVirtualization of project packages complete!");

	Output = MakeUnique<FVirtualizeCommandOutput>(Project.GetProjectName(), Result.DescriptionTags);

	return true;
}

bool FVirtualizeCommand::ProcessOutput(const TArray<TUniquePtr<FCommandOutput>>& CmdOutputArray)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FVirtualizeCommand::ProcessOutput);

	if (bShouldSubmitChangelist)
	{
		TArray<FString> FinalDescriptionTags = BuildFinalTagDescriptions(CmdOutputArray);

		if (!TrySubmitChangelist(SourceChangelist, FinalDescriptionTags))
		{
			return false;
		}
	}

	return true;
}

TUniquePtr<FCommandOutput> FVirtualizeCommand::CreateOutputObject() const
{
	return MakeUnique<FVirtualizeCommandOutput>();
}

const TArray<FString>& FVirtualizeCommand::GetPackages() const
{
	return AllPackages;
}

bool FVirtualizeCommand::TrySubmitChangelist(const FSourceControlChangelistPtr& ChangelistToSubmit, const TArray<FString>& InDescriptionTags)
{
	if (!ChangelistToSubmit.IsValid())
	{
		return true;
	}

	if (!TryConnectToSourceControl(ClientSpecName))
	{
		UE_LOGF(LogVirtualizationTool, Error, "Submit failed, cannot find a valid source control provider");
		return false;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FVirtualizeCommand::TrySubmitChangelist);

	FSourceControlChangelistRef Changelist = ChangelistToSubmit.ToSharedRef();
	FSourceControlChangelistStatePtr ChangelistState = SCCProvider->GetState(Changelist, EStateCacheUsage::Use);

	if (!ChangelistState.IsValid())
	{
		UE_LOGF(LogVirtualizationTool, Error, "Submit failed, failed to find the state for the changelist");
		return false;
	}

	const FString ChangelistNumber = ChangelistState->GetDisplayText().ToString();

	UE_LOGF(LogVirtualizationTool, Display, "Attempting to submit the changelist '%ls'", *ChangelistNumber);

	TSharedRef<FCheckIn, ESPMode::ThreadSafe> CheckInOperation = ISourceControlOperation::Create<FCheckIn>();

	// Grab the original changelist description then append our tags afterwards
	TStringBuilder<512> Description;
	Description << ChangelistState->GetDescriptionText().ToString();

	for (const FString& Tag : InDescriptionTags)
	{
		Description << TEXT("\n") << Tag;
	}

	CheckInOperation->SetDescription(FText::FromString(Description.ToString()));

	if (SCCProvider->Execute(CheckInOperation, Changelist) == ECommandResult::Succeeded)
	{
		UE_LOGF(LogVirtualizationTool, Display, "%ls", *CheckInOperation->GetSuccessMessage().ToString());
		return true;
	}
	else
	{
		// Even when log suppression is active we still show errors to the users and as the source control
		// operation should have logged the problem as an error the user will see it. This means we don't 
		// have to extract it from CheckInOperation .
		UE_LOGF(LogVirtualizationTool, Error, "Submit failed, please check the log!");

		return false;
	}
}

bool FVirtualizeCommand::TryParsePackageList(const FString& PackageListPath, TArray<FString>& OutPackages)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FVirtualizeCommand::TryParsePackageList);

	UE_LOGF(LogVirtualizationTool, Display, "Parsing the package list '%ls'...", *PackageListPath);

	if (!IFileManager::Get().FileExists(*PackageListPath))
	{
		UE_LOGF(LogVirtualizationTool, Error, "\tThe package list '%ls' does not exist", *PackageListPath);
		return false;
	}

	if (FFileHelper::LoadFileToStringArray(OutPackages, *PackageListPath))
	{
		// We don't have control over how the package list was generated so make sure that the paths
		// are in the format that we want.
		for (FString& PackagePath : OutPackages)
		{
			FPaths::NormalizeFilename(PackagePath);
		}

		return true;
	}
	else
	{
		UE_LOGF(LogVirtualizationTool, Error, "\tFailed to parse the package list '%ls'", *PackageListPath);
		return false;
	}
}


FVirtualizeLegacyChangeListCommand::FVirtualizeLegacyChangeListCommand(FStringView CommandName)
	: FVirtualizeCommand(CommandName)
{

}

bool FVirtualizeLegacyChangeListCommand::Initialize(const TCHAR* CmdLine)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FVirtualizeLegacyChangeListCommand::Initialize);

	UE_LOGF(LogVirtualizationTool, Warning, "Using legacy -Mode=Changelist command, use '-Mode=Virtualization -Changelist=123' instead!");

	if (!FParse::Value(CmdLine, TEXT("-ClientSpecName="), ClientSpecName))
	{
		UE_LOGF(LogVirtualizationTool, Error, "Failed to find cmdline switch 'ClientSpecName', this is a required parameter!");
		return false;
	}

	if (!FParse::Value(CmdLine, TEXT("-Changelist="), SourceChangelistNumber))
	{
		UE_LOGF(LogVirtualizationTool, Error, "Failed to find cmdline switch 'Changelist', this is a required parameter!");
		return false;

	}

	if (!TryParseChangelist(ClientSpecName, SourceChangelistNumber, AllPackages, &SourceChangelist))
	{
		return false;
	}

	if (FParse::Param(CmdLine, TEXT("NoSubmit")))
	{
		UE_LOGF(LogVirtualizationTool, Display, "Cmdline parameter '-NoSubmit' found, the changelist will be virtualized but not submitted!");
		bShouldSubmitChangelist = false;
	}
	else
	{
		// The legacy command was opt out when it came to submitting the changelist, we need to maintain those defaults
		bShouldSubmitChangelist = true;
	}

	return true;
}

FVirtualizeLegacyPackageListCommand::FVirtualizeLegacyPackageListCommand(FStringView CommandName)
	: FVirtualizeCommand(CommandName)
{

}

bool FVirtualizeLegacyPackageListCommand::Initialize(const TCHAR* CmdLine)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FVirtualizeLegacyPackageListCommand::Initialize);

	UE_LOGF(LogVirtualizationTool, Warning, "Using legacy -Mode=Packagelist command, use '-Mode=Virtualization -Path=PathToFile' instead!");

	FString PackageListPath;
	if (FParse::Value(CmdLine, TEXT("-Path="), PackageListPath))
	{
		return TryParsePackageList(PackageListPath, AllPackages);
	}
	else
	{

		return false;
	}
}

} // namespace UE::Virtualization
