// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Commands/CommandBase.h"

namespace UE::Virtualization
{

struct FVirtualizeCommandOutput final : public FCommandOutput
{
	FVirtualizeCommandOutput() = default;
	FVirtualizeCommandOutput(FStringView InProjectName, const TArray<FText>& InDescriptionTags);

	BEGIN_JSON_SERIALIZER
		JSON_SERIALIZE_PARENT(FCommandOutput);
		JSON_SERIALIZE_ARRAY("DescriptionTags", DescriptionTags);
	END_JSON_SERIALIZER

	TArray<FString> DescriptionTags;
};

class FVirtualizeCommand : public FCommand
{
public:
	FVirtualizeCommand(FStringView CommandName);
	virtual ~FVirtualizeCommand() = default;

	static void PrintCmdLineHelp();

private:
	virtual bool Initialize(const TCHAR* CmdLine) override;

	virtual void Serialize(FJsonSerializerBase& Serializer) override;

	virtual bool IsProjectValidForCommand(const FProject& Project) const final;

	virtual bool ProcessProject(const FProject& Project, TUniquePtr<FCommandOutput>& Output) override;
	virtual bool ProcessOutput(const TArray<TUniquePtr<FCommandOutput>>& CmdOutputArray) override;

	virtual TUniquePtr<FCommandOutput> CreateOutputObject() const override;

	virtual const TArray<FString>& GetPackages() const override;

	bool TrySubmitChangelist(const FSourceControlChangelistPtr& ChangelistToSubmit, const TArray<FString>& DescriptionTags);
	
protected:

	bool TryParsePackageList(const FString& PackageListPath, TArray<FString>& OutPackages);	

protected:
	FString ClientSpecName;

	TArray<FString> AllPackages;

	FSourceControlChangelistPtr SourceChangelist;
	FString SourceChangelistNumber;

	bool bShouldSubmitChangelist = false;
	bool bShouldCheckout = false;
};

/** Used to convert the legacy command line -Mode=Changelist to the new style of command */
class FVirtualizeLegacyChangeListCommand final : public FVirtualizeCommand
{
public:
	FVirtualizeLegacyChangeListCommand(FStringView CommandName);
	virtual ~FVirtualizeLegacyChangeListCommand() = default;

private:
	virtual bool Initialize(const TCHAR* CmdLine) override;
};

/** Used to convert the legacy command line -Mode=Packagelist to the new style of command */
class FVirtualizeLegacyPackageListCommand final : public FVirtualizeCommand
{
public:
	FVirtualizeLegacyPackageListCommand(FStringView CommandName);
	virtual ~FVirtualizeLegacyPackageListCommand() = default;

private:
	virtual bool Initialize(const TCHAR* CmdLine) override;
};

} // namespace UE::Virtualization
