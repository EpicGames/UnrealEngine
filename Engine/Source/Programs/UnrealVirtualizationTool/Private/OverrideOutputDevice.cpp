// Copyright Epic Games, Inc. All Rights Reserved.

#include "OverrideOutputDevice.h"

#include "Logging/StructuredLog.h"
#include "Misc/CoreDelegates.h"
#include "UnrealVirtualizationTool.h"

namespace UE
{

FOverrideOutputDevice::FOverrideOutputDevice()
{
	OriginalLog = GWarn;
	GWarn = this;

	// If this was created before the output devices were initialized we need to register for a
	// callback when they are initialized so that we can override them.
	if (OriginalLog == nullptr)
	{
		OnInitHandle = FCoreDelegates::OnOutputDevicesInit.AddLambda([this]()
			{
				OriginalLog = GWarn;
				GWarn = this;
			});
	}
}

FOverrideOutputDevice::~FOverrideOutputDevice()
{
	GWarn = OriginalLog;

	if (OnInitHandle.IsValid())
	{
		FCoreDelegates::OnOutputDevicesInit.Remove(OnInitHandle);
		OnInitHandle.Reset();
	}
}

void FOverrideOutputDevice::Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category, double Time)
{
#if !NO_LOGGING
	if (ShouldFilterMessage(Verbosity, Category))
	{
		Verbosity = ELogVerbosity::Log;
	}
#endif // !NO_LOGGING

	FFeedbackContextAnsi::Serialize(V, Verbosity, Category, Time);
}

void FOverrideOutputDevice::SerializeRecord(const UE::FLogRecord& Record)
{
#if !NO_LOGGING
	if (ShouldFilterMessage(Record.GetVerbosity(), Record.GetCategory()))
	{
		UE::FLogRecord LocalRecord = Record;
		LocalRecord.SetVerbosity(ELogVerbosity::Log);
		return FFeedbackContextAnsi::SerializeRecord(LocalRecord);
	}
#endif // !NO_LOGGING

	FFeedbackContextAnsi::SerializeRecord(Record);
}

bool FOverrideOutputDevice::ShouldFilterMessage(ELogVerbosity::Type Verbosity, const FName& Category)
{
#if !NO_LOGGING
	// We only want 'LogVirtualizationTool' messages in display
	if (Verbosity == ELogVerbosity::Display && Category != LogVirtualizationTool.GetCategoryName())
	{
		return true;
	}

	// Suppress errors from our reporting systems
	if (Verbosity == ELogVerbosity::Error && Category == LogOutputDevice.GetCategoryName())
	{
		return true;
	}
#endif //  !NO_LOGGING

	return false;
}

} // namespace UE
