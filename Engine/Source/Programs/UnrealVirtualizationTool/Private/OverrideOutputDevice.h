// Copyright Epic Games, Inc. All Rights Reserved.

#include "Delegates/IDelegateInstance.h"
#include "HAL/FeedbackContextAnsi.h"

#pragma once

namespace UE { class FLogRecord; }

namespace UE
{
/**
 * This class can be used to prevent log messages from other systems being logged with the Display verbosity.
 * In practical terms this means as long as the class is alive, only LogVirtualizationTool messages will
 * be logged to the display meaning the user will have less information to deal with.
 */
class FOverrideOutputDevice final : public FFeedbackContextAnsi
{
public:
	FOverrideOutputDevice();

	virtual ~FOverrideOutputDevice();

	virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category) override
	{
		Serialize(V, Verbosity, Category, -1.0);
	}

	virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category, double Time) override;

	virtual void SerializeRecord(const UE::FLogRecord& Record) override;

private:

	bool ShouldFilterMessage(ELogVerbosity::Type Verbosity, const FName& Category);

	FFeedbackContext* OriginalLog;
	FDelegateHandle OnInitHandle;
};

} // namespace UE
