// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "HAL/PlatformProcess.h"
#include "Logging/LogMacros.h"
#include "UnrealVirtualizationTool.h"

namespace UE::Virtualization
{

/** Wrapper around stdin and stdout pipes created by a call to FPlatformProcess::CreatePipe */
struct FProcessPipes
{
	FProcessPipes()
	{
		verify(FPlatformProcess::CreatePipe(StdOutReadPipe, StdOutWritePipe, false));
		verify(FPlatformProcess::CreatePipe(StdInReadPipe, StdInWritePipe, true));
	}

	~FProcessPipes()
	{
		FPlatformProcess::ClosePipe(StdOutReadPipe, StdOutWritePipe);
		FPlatformProcess::ClosePipe(StdInReadPipe, StdInWritePipe);
	}

	void ProcessStdOut()
	{
		check(GetStdOutForReading() != nullptr);

		FString Output = FPlatformProcess::ReadPipe(GetStdOutForReading());

		while (!Output.IsEmpty())
		{
			TArray<FString> Lines;
			Output.ParseIntoArray(Lines, LINE_TERMINATOR);

			for (const FString& Line : Lines)
			{
				UE_LOGF(LogVirtualizationTool, Display, "Child Process-> %ls", *Line);
			}

			Output = FPlatformProcess::ReadPipe(GetStdOutForReading());
		}
	}

	void* GetStdInForProcess() const
	{
		return StdInReadPipe;
	}

	void* GetStdInForWriting() const
	{
		return StdInWritePipe;
	}

	void* GetStdOutForProcess() const
	{
		return StdOutWritePipe;
	}

	void* GetStdOutForReading() const
	{
		return StdOutReadPipe;
	}

private:

	void* StdOutReadPipe = nullptr;
	void* StdOutWritePipe = nullptr;

	void* StdInReadPipe = nullptr;
	void* StdInWritePipe = nullptr;
};

} // namespace UE::Virtualization