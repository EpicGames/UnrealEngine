// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnrealVirtualizationTool.h"

#include "Modules/ModuleManager.h"
#include "ProjectUtilities.h"
#include "RequiredProgramMainCPPInclude.h"
#include "UnrealVirtualizationToolApp.h"

#include "OverrideOutputDevice.h"

IMPLEMENT_APPLICATION(UnrealVirtualizationTool, "UnrealVirtualizationTool");

DEFINE_LOG_CATEGORY(LogVirtualizationTool);

/**
 * Used to search the raw command line to see if it contains a specific switch.
 * This is the same as calling FParse::Param(FCommandLine::Get(), Switch) except
 * it will work before FCommandLine has initialized.
 * 
 * Note that like the example code above we expect the caller to omit the '-'
 * character from the switch being searched for.
 * For example if the user is searching for '-Example' we would expect them to
 * call this function with TEXT("Example").
 * 
 * @param Switch The command line switch to search for
 * @param ArgC The number of args provided
 * @param ArgV An array of args provided
 * 
 * @return True if the requested switch was found, otherwise false.
 */
static bool DoesSwitchExist(FStringView Switch, int32 ArgC, TCHAR* ArgV[])
{
	// Skip the first arg as it will just be the path of the exe ( see BuildFromArgVImpl)
	for (int32 Index = 1; Index < ArgC; ++Index)
	{
		// Skip the first character of each arg as each switch should start with '-'
		FStringView Arg(ArgV[Index]);
		Arg.RightChopInline(1);

		if (Switch == Arg)
		{
			return true;
		}
	}

	return false;
}

int32 UnrealVirtualizationToolMain(int32 ArgC, TCHAR* ArgV[])
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UnrealVirtualizationToolMain);

	using namespace UE::Virtualization;

	// Allows this program to accept a project argument on the commandline and use project-specific config
	UE::ProjectUtilities::ParseProjectDirFromCommandline(ArgC, ArgV);

	TUniquePtr<UE::FOverrideOutputDevice> OutputDeviceOverride;
	if (DoesSwitchExist(TEXTVIEW("MinimalLogging"), ArgC, ArgV))
	{
		OutputDeviceOverride = MakeUnique<UE::FOverrideOutputDevice>();
	}

	GEngineLoop.PreInit(ArgC, ArgV);
	check(GConfig && GConfig->IsReadyForUse());

	const bool bReportFailures = FParse::Param(FCommandLine::Get(), TEXT("ReportFailures"));
	
#if 0
	while (!FPlatformMisc::IsDebuggerPresent())
	{
		FPlatformProcess::SleepNoStats(0.0f);
	}

	PLATFORM_BREAK();
#endif

	FModuleManager::Get().StartProcessingNewlyLoadedObjects();

	UE_LOGF(LogVirtualizationTool, Display, "Running UnrealVirtualization Tool");

	EProcessResult ProcessResult = EProcessResult::Success;

	FUnrealVirtualizationToolApp App;

	EInitResult InitResult = App.Initialize();
	if (InitResult == EInitResult::Success)
	{
		ProcessResult = App.Run();
		if (ProcessResult != EProcessResult::Success)
		{
			UE_LOGF(LogVirtualizationTool, Error, "UnrealVirtualizationTool ran with errors");
		}
	}	
	else if(InitResult == EInitResult::Error)
	{
		UE_LOGF(LogVirtualizationTool, Error, "UnrealVirtualizationTool failed to initialize");
		ProcessResult = EProcessResult::Error;
	}

	UE_CLOGF(ProcessResult == EProcessResult::Success, LogVirtualizationTool, Display, "UnrealVirtualizationTool ran successfully");

	// Don't report if the error was in a child process, they will raise their own ensures
	if (bReportFailures && ProcessResult == EProcessResult::Error)
	{
		ensure(false);
	}

	const uint8 ReturnCode = ProcessResult == EProcessResult::Success ? 0 : 1;

	if (FParse::Param(FCommandLine::Get(), TEXT("fastexit")))
	{
		FPlatformMisc::RequestExitWithStatus(true, ReturnCode);
	}
	else
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Shutdown);

		GConfig->DisableFileOperations(); // We don't want to write out any config file changes!

		// Even though we are exiting anyway we need to request an engine exit in order to get a clean shutdown
		RequestEngineExit(TEXT("The process has finished"));

		FEngineLoop::AppPreExit();
		FModuleManager::Get().UnloadModulesAtShutdown();
		FEngineLoop::AppExit();
	}

	return ReturnCode;
}

INT32_MAIN_INT32_ARGC_TCHAR_ARGV()
{
	FTaskTagScope Scope(ETaskTag::EGameThread);
	return UnrealVirtualizationToolMain(ArgC, ArgV);
}