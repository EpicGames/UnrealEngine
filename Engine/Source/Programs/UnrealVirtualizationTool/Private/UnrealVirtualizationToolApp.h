// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "ProjectFiles.h"
#include "Templates/UniquePtr.h"

namespace UE::Virtualization
{

class FCommand;
struct FCommandOutput;

/** The mode of the application to run */
enum class EMode :uint32
{
	/** Error condition */
	Unknown = 0,

	// Legacy (should be phased out)

	/** Virtualize and submit a given changelist */
	Changelist,
	/** Virtualize a list of packages provided by text file */
	PackageList,

	/// New Style Commands

	/** Virtualize one or more packages */
	Virtualize,

	/** Rehydrate one or more packages */
	Rehydrate
};

/** The result of FUnrealVirtualizationToolApp initialization */
enum class EInitResult
{
	/** Initialization succeeded and the application should run. */
	Success = 0,
	/** No error was encountered but the application should early out and not run. */
	EarlyOut,
	/** Initialization failed with an error, we should not run and should display an error to the user */
	Error
};

enum class EProcessResult
{
	Success = 0,
	Error,
	ChildProcessError
};

class FUnrealVirtualizationToolApp
{
public:
	FUnrealVirtualizationToolApp();
	~FUnrealVirtualizationToolApp();

	EInitResult Initialize();
	EProcessResult Run();

private:

	void PrintCmdLineHelp() const;

	EProcessResult ProcessProjects(TArray<TUniquePtr<FCommandOutput>>& OutputArray);

	bool TryLoadModules();
	bool TryInitEnginePlugins();
	
	EInitResult TryParseCmdLine();
	EInitResult TryParseGlobalOptions(const TCHAR* CmdLine);
	
	EInitResult CreateCommandFromString(const FString& CommandName, const TCHAR* Cmdline);

	bool TrySortFilesByProject(const TArray<FString>& Packages);
	void FilterProjectsForCommand();

	bool TryFindProject(const FString& PackagePath, FString& ProjectFilePath, FString& PluginFilePath) const;
	FProject& FindOrAddProject(FString&& ProjectFilePath);

	bool IsChildProcess() const;

	void AddGlobalOption(FStringView Options);

	static bool TryReadChildProcessOutputFile(const FGuid& ChildProcessId, const FCommand& Command, TArray<TUniquePtr<FCommandOutput>>& OutputArray);	
	static bool TryWriteChildProcessOutputFile(const FString& ChildProcessId, const TArray<TUniquePtr<FCommandOutput>>& OutputArray);
	
	/** Note not currently static due to dependencies */
	bool TryReadChildProcessInputFile(const FString& InputPath);
	static bool TryWriteChildProcessInputFile(const FGuid& ChildProcessId, const FCommand& Command, const FProject& Project, FStringBuilderBase& OutPath);
	static void CleanUpChildProcessFiles(const FGuid& ChildProcessId);

	static EProcessResult LaunchChildProcess(const FCommand& Command, const FProject& Project, FStringView GlobalOptions, TArray<TUniquePtr<FCommandOutput>>& OutputArray);

private:

	/** Override used to suppress log messages */
	TUniquePtr<FFeedbackContext> OutputDeviceOverride;

	/** Stored the id of the child process. This is the base file name of the input file used to launch it */
	FString ChildProcessId;

	/** A string containing all global options that were found in the cmdline */
	FString GlobalCmdlineOptions;

	/** Data structure holding the files in the changelist sorted by project and then by plugin*/
	TArray<FProject> Projects;

	/** The mode for the application to run */
	EMode Mode = EMode::Unknown;

	/** The command being processed */
	TUniquePtr<FCommand> CurrentCommand;
};

} //namespace UE::Virtualization
