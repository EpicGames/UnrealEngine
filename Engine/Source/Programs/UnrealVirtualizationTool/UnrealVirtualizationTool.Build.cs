// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class UnrealVirtualizationTool : ModuleRules
{
	public UnrealVirtualizationTool(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Virtualization",
				"Projects",
				"ApplicationCore",
				"SourceControl",
				"PerforceSourceControl",
				"DesktopPlatform",
				"Json"
			});

		PublicIncludePathModuleNames.Add("Launch");
	}
}
