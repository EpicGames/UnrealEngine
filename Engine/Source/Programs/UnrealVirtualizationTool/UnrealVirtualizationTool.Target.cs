// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.Collections.Generic;

public class UnrealVirtualizationToolTarget : TargetRules
{
	public UnrealVirtualizationToolTarget(TargetInfo Target) : base(Target)
	{
		Type = TargetType.Program;
		LinkType = TargetLinkType.Monolithic;
		IncludeOrderVersion = EngineIncludeOrderVersion.Latest;

		LaunchModuleName = "UnrealVirtualizationTool";

		bBuildDeveloperTools = false;

		bCompileAgainstEditor = false;
		bBuildWithEditorOnlyData = true;

		// Currently this app is not linking against the engine, so we'll compile out references from Core to the rest of the engine
		bCompileAgainstApplicationCore = true;
		bCompileAgainstCoreUObject = true;
		bCompileAgainstEngine = false;
		bUsesSlate = false;

		// We don't use these and they are causing 'lld-link: warning: found both wmain and main; using latter' warnings with clang
		bCompileFreeType = false;
		bCompileICU = false;

		// This app is a console application, not a Windows app (sets entry point to main(), instead of WinMain())
		bIsBuildingConsoleApplication = true;

		// Enable Developer plugins
		bCompileWithPluginSupport = true;

		// Reduce the final exe/pdb size
		WindowsPlatform.bStripUnreferencedSymbols = true;
		WindowsPlatform.bMergeIdenticalCOMDATs = true;
		WindowsPlatform.bUseBundledDbgHelp = false;
		WindowsPlatform.bPixProfilingEnabled = false;

		bEnableTrace = true;
		GlobalDefinitions.Add("UE_SUPPORT_FULL_PACKAGEPATH=1");
	}
}
